// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualStudioCodeSourceCodeAccessModule.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "VisualStudioCodeSourceCodeAccessor.h"

LLM_DEFINE_TAG(VisualStudioCodeSourceCodeAccess);

IMPLEMENT_MODULE( FVisualStudioCodeSourceCodeAccessModule, VisualStudioCodeSourceCodeAccess );

#define LOCTEXT_NAMESPACE "VisualStudioCodeSourceCodeAccessor"

FVisualStudioCodeSourceCodeAccessModule::FVisualStudioCodeSourceCodeAccessModule()
	: VisualStudioCodeSourceCodeAccessor(MakeShareable(new FVisualStudioCodeSourceCodeAccessor()))
{
}

void FVisualStudioCodeSourceCodeAccessModule::StartupModule()
{
	LLM_SCOPE_BYTAG(VisualStudioCodeSourceCodeAccess);

	VisualStudioCodeSourceCodeAccessor->Startup();

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &VisualStudioCodeSourceCodeAccessor.Get() );
}

void FVisualStudioCodeSourceCodeAccessModule::ShutdownModule()
{
	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &VisualStudioCodeSourceCodeAccessor.Get());

	VisualStudioCodeSourceCodeAccessor->Shutdown();
}

FVisualStudioCodeSourceCodeAccessor& FVisualStudioCodeSourceCodeAccessModule::GetAccessor()
{
	return VisualStudioCodeSourceCodeAccessor.Get();
}

#undef LOCTEXT_NAMESPACE
