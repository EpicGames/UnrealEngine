// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualStudioCodeSourceCodeAccessor.h"
#include "VisualStudioCodeSourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"
#include "Misc/UProjectInfo.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if PLATFORM_WINDOWS
#include "Internationalization/Regex.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogVSCodeAccessor, Log, All);

#define LOCTEXT_NAMESPACE "VisualStudioCodeSourceCodeAccessor"

namespace
{
	static const TCHAR* GVSCodeWorkspaceExtension = TEXT(".code-workspace");
}

static FString MakePath(const FString& InPath)
{
	return TEXT("\"") + InPath.ReplaceQuotesWithEscapedQuotes() + TEXT("\""); 
}

FString FVisualStudioCodeSourceCodeAccessor::GetSolutionPath() const
{
	FScopeLock Lock(&CachedSolutionPathCriticalSection);

	if (IsInGameThread())
	{
		FString PrimaryProjectPath;
		if (!FFileHelper::LoadFileToString(PrimaryProjectPath, *(FPaths::EngineIntermediateDir() / TEXT("ProjectFiles") / TEXT("PrimaryProjectPath.txt"))) || 
			(PrimaryProjectPath.TrimStartAndEndInline(), PrimaryProjectPath.IsEmpty()))
		{
			const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
			const FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

			if ((CurrentProject == nullptr || CurrentProject->Modules.Num() == 0) || !FUProjectDictionary::GetDefault().IsForeignProject(ProjectDir))
			{
				PrimaryProjectPath = FPaths::RootDir() / TEXT("UE5");
			}
			else
			{
				const FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(ProjectDir);
				PrimaryProjectPath = ProjectDir / BaseName;
			}
		}	

		CachedSolutionPath = PrimaryProjectPath + GVSCodeWorkspaceExtension;
		FPaths::NormalizeFilename(CachedSolutionPath);			
	}
	
	return CachedSolutionPath;
}

/** save all open documents in visual studio, when recompiling */
static void OnModuleCompileStarted(bool bIsAsyncCompile)
{
	FVisualStudioCodeSourceCodeAccessModule& VisualStudioCodeSourceCodeAccessModule = FModuleManager::LoadModuleChecked<FVisualStudioCodeSourceCodeAccessModule>(TEXT("VisualStudioCodeSourceCodeAccess"));
	VisualStudioCodeSourceCodeAccessModule.GetAccessor().SaveAllOpenDocuments();
}

void FVisualStudioCodeSourceCodeAccessor::Startup()
{
	GetSolutionPath();
	RefreshAvailability();
}

void FVisualStudioCodeSourceCodeAccessor::RefreshAvailability()
{
#if PLATFORM_WINDOWS
	FString IDEPath;

	if (!FWindowsPlatformMisc::QueryRegKey(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Classes\\Applications\\Code.exe\\shell\\open\\command\\"), TEXT(""), IDEPath))
	{
		FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Classes\\Applications\\Code.exe\\shell\\open\\command\\"), TEXT(""), IDEPath);
	}

	FString PatternString(TEXT("\"(.*)\" \".*\""));
	FRegexPattern Pattern(PatternString);
	FRegexMatcher Matcher(Pattern, IDEPath);
	if (Matcher.FindNext())
	{
		FString URL = Matcher.GetCaptureGroup(1);
		if (FPaths::FileExists(URL))
		{
			Location.URL = URL;
		}
	}
#elif PLATFORM_LINUX
	FString OutURL;
	int32 ReturnCode = -1;

	FPlatformProcess::ExecProcess(TEXT("/usr/bin/env"), TEXT("bash -c \"type -p code\""), &ReturnCode, &OutURL, nullptr);
	if (ReturnCode == 0)
	{
		Location.URL = OutURL.TrimStartAndEnd();
	}
	else
	{
		// Fallback to default install location
		FString URL = TEXT("/usr/bin/code");
		if (FPaths::FileExists(URL))
		{
			Location.URL = URL;
		}
	}
#elif PLATFORM_MAC
	NSURL* AppURL = [[NSWorkspace sharedWorkspace] URLForApplicationWithBundleIdentifier:@"com.microsoft.VSCode"];
	if (AppURL != nullptr)
	{
		Location.URL = FString([AppURL path]);
	}
#endif
}

void FVisualStudioCodeSourceCodeAccessor::Shutdown()
{
}

bool FVisualStudioCodeSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if (Location.IsValid())
	{
		FString SolutionDir = GetSolutionPath();
		TArray<FString> Args;
		Args.Add(MakePath(SolutionDir));

		for (const FString& SourcePath : AbsoluteSourcePaths)
		{
			Args.Add(MakePath(SourcePath));
		}

		return Launch(Args);
	}

	return false;
}

bool FVisualStudioCodeSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	// VSCode doesn't need to do anything when new files are added
	return true;
}

bool FVisualStudioCodeSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	if (Location.IsValid())
	{
		// Column & line numbers are 1-based, so dont allow zero
		LineNumber = LineNumber > 0 ? LineNumber : 1;
		ColumnNumber = ColumnNumber > 0 ? ColumnNumber : 1;

		FString SolutionDir = GetSolutionPath();
		TArray<FString> Args;
		Args.Add(MakePath(SolutionDir));
		Args.Add(TEXT("-g"));
		Args.Add(MakePath(FullPath + FString::Printf(TEXT(":%d:%d"), LineNumber, ColumnNumber)));
		return Launch(Args);
	}

	return false;
}

bool FVisualStudioCodeSourceCodeAccessor::CanAccessSourceCode() const
{
	// True if we have any versions of VS installed
	return Location.IsValid();
}

FName FVisualStudioCodeSourceCodeAccessor::GetFName() const
{
	return FName("VisualStudioCode");
}

FText FVisualStudioCodeSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("VisualStudioDisplayName", "Visual Studio Code");
}

FText FVisualStudioCodeSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("VisualStudioDisplayDesc", "Open source code files in Visual Studio Code");
}

void FVisualStudioCodeSourceCodeAccessor::Tick(const float DeltaTime)
{
}

bool FVisualStudioCodeSourceCodeAccessor::OpenSolution()
{
	if (Location.IsValid())
	{
		return OpenSolutionAtPath(GetSolutionPath());
	}

	return false;
}

bool FVisualStudioCodeSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	if (Location.IsValid())
	{
		FString SolutionPath = InSolutionPath;

		if (!SolutionPath.EndsWith(GVSCodeWorkspaceExtension))
		{
			SolutionPath = SolutionPath + GVSCodeWorkspaceExtension;
		}

		TArray<FString> Args;
		Args.Add(MakePath(SolutionPath));
		return Launch(Args);
	}

	return false;
}

bool FVisualStudioCodeSourceCodeAccessor::DoesSolutionExist() const
{
	return FPaths::FileExists(GetSolutionPath());
}


bool FVisualStudioCodeSourceCodeAccessor::SaveAllOpenDocuments() const
{
	return true;
}

bool FVisualStudioCodeSourceCodeAccessor::Launch(const TArray<FString>& InArgs)
{
	if (Location.IsValid())
	{
		FString ArgsString;
		for (const FString& Arg : InArgs)
		{
			ArgsString.Append(Arg);
			ArgsString.Append(TEXT(" "));
		}

		uint32 ProcessID;
		FProcHandle hProcess = FPlatformProcess::CreateProc(*Location.URL, *ArgsString, true, false, false, &ProcessID, 0, nullptr, nullptr, nullptr);
		if (hProcess.IsValid())
		{
			FPlatformProcess::CloseProc(hProcess);
			return true;
		}
		else
		{
			return false;
		}
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
