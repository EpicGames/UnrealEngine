// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ISourceCodeAccessor.h"

class FVisualStudioCodeSourceCodeAccessor : public ISourceCodeAccessor
{
public:
	FVisualStudioCodeSourceCodeAccessor()
	{
	}

	/** Initialise internal systems, register delegates etc. */
	void Startup();

	/** Shut down internal systems, unregister delegates etc. */
	void Shutdown();

	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() override;
	virtual bool CanAccessSourceCode() const override;
	virtual FName GetFName() const override;
	virtual FText GetNameText() const override;
	virtual FText GetDescriptionText() const override;
	virtual bool OpenSolution() override;
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;
	virtual bool DoesSolutionExist() const override;
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;
	virtual bool SaveAllOpenDocuments() const override;
	virtual void Tick(const float DeltaTime) override;

private:

	/** Wrapper for vscode executable launch information */
	struct FLocation
	{
		bool IsValid() const
		{
			return URL.Len() > 0;
		}

		FString URL;
	};

	/** Location instance */
	FLocation Location;

	/** String storing the solution path obtained from the module manager to avoid having to use it on a thread */
	mutable FString CachedSolutionPath;

	/** Critical section for updating SolutionPath */
	mutable FCriticalSection CachedSolutionPathCriticalSection;

	/** Accessor for SolutionPath. Will try to update it when called from the game thread, otherwise will use the cached value */
	FString GetSolutionPath() const;

	/** Helper function for launching the VSCode instance with the given list of arguments */
	bool Launch(const TArray<FString>& InArgs);
};
