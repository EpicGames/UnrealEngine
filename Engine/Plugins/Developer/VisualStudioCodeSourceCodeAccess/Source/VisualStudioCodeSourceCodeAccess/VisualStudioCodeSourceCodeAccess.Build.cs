// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using Microsoft.Win32;

namespace UnrealBuildTool.Rules
{
	public class VisualStudioCodeSourceCodeAccess : ModuleRules
	{
		public VisualStudioCodeSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SourceCodeAccess",
					"DesktopPlatform",
					"Projects",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("HotReload");
			}

			bool bHasVisualStudioDTE = false;
			try
			{
				{
					// Interrogate the Win32 registry
					string DTEKey = null;
					switch (Target.WindowsPlatform.ToolChain)
					{
						case WindowsCompiler.VisualStudio2022:
							DTEKey = "VisualStudio.DTE.17.0";
							break;
						default:
							throw new Exception("Unknown visual studio version when mapping to DTEKey: " +
												Target.WindowsPlatform.ToolChain.ToString());
					}
					if (OperatingSystem.IsWindows())
					{
						bHasVisualStudioDTE = RegistryKey.OpenBaseKey(RegistryHive.ClassesRoot, RegistryView.Registry32).OpenSubKey(DTEKey) != null;
					}
				}
			}
			catch
			{
				bHasVisualStudioDTE = false;
			}

			if (bHasVisualStudioDTE)
			{
				PublicDefinitions.Add("VSACCESSOR_HAS_DTE=1");
			}
			else
			{
				PublicDefinitions.Add("VSACCESSOR_HAS_DTE=0");
			}

			bBuildLocallyWithSNDBS = true;

			ShortName = "VSCSCA";
		}
	}
}
